// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioModulationInsights : ModuleRules
{
	public AudioModulationInsights(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioInsights",
				"AudioMixerCore",
				"CoreUObject",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioInsightsEditor",
					"Engine",
				}
			);
		}
	}
}
