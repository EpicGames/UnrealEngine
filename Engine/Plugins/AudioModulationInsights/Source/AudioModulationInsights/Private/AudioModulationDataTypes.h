// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

namespace AudioModulationInsights
{
	/*
	* Note: The following mirrors data from Engine/Plugins/Runtime/AudioModulation
	*
	* Duplication is required as AudioInsights cannot depend on any engine code
	* As it must be able to run in standalone (where engine code is not available)
	* 
	* Caution: Since this means we no longer have a single source of truth
	* We must maintain this data if the underlying engine types change
	*/

	using FBusId = uint32;								// mirrors FBusId: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/SoundControlBusProxy.h
	using FModulatorId = uint32;						// mirrors FModulatorId: Engine/Source/Runtime/AudioExtensions/Public/IAudioModulation.h
	using FModulatorTypeId = uint32;					// mirrors FModulatorTypeId: Engine/Source/Runtime/AudioExtensions/Public/IAudioModulation.h
	enum class EModulatorTraceType : FModulatorTypeId	// mirrors EModulatorTraceType: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/AudioModulationSystem.h
	{
		ControlBus,
		ControlBusMix,
		Generator,
		ParameterPatch,

		COUNT
	};

	static inline FText GetModulatorTraceTypeName(const EModulatorTraceType InModulatorTraceType)
	{
		switch (InModulatorTraceType) 
		{
			case EModulatorTraceType::ControlBus:		return LOCTEXT("AudioModulation_ModulatorTypeControlBus", "Control Bus");
			case EModulatorTraceType::ControlBusMix:	return LOCTEXT("AudioModulation_ModulatorTypeBusMix", "Bus Mix");
			case EModulatorTraceType::Generator:		return LOCTEXT("AudioModulation_ModulatorTypeGenerator", "Generator");
			case EModulatorTraceType::ParameterPatch:	return LOCTEXT("AudioModulation_ModulatorTypePatch", "Parameter Patch");
			case EModulatorTraceType::COUNT:			break;
		}

		return FText::GetEmpty();
	}

	// Column names used across multiple view factories
	namespace ModulatorColumnNames
	{
		const FName ModulatorIDColumnName{"ModulatorID"};
		const FName ModulatorNameColumnName{"ModulatorName"};
		const FName ModulatorTypeColumnName{"ModulatorType"};
		const FName ModulatorValueColumnName{"ModulatorValue"};
		const FName ModulatorBypassedColumnName{"Bypassed"};
	}

	namespace ModulatorColors
	{
		const FSlateColor Default = FSlateColor(FColor::White);
		const FSlateColor SelectedModulator = FSlateColor(FColor::Green);
		const FSlateColor SelectedContributor = FSlateColor(FColor::Yellow);
	}

	static const FModulatorId InvalidModulatorId = -1;

} // namespace AudioModulationInsights

#undef LOCTEXT_NAMESPACE
