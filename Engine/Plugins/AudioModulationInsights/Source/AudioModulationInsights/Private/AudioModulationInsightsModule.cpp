// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationInsightsModule.h"

#include "IAudioInsightsModule.h"

#if WITH_EDITOR
#include "IAudioInsightsEditorModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

void FAudioModulationInsightsModule::StartupModule()
{
	// Don't run providers in any commandlet to avoid additional, unnecessary overhead as audio modulation insights is dormant.
	if (!IsRunningCommandlet())
	{
#if WITH_EDITOR
		IAudioInsightsEditorModule& AudioInsightsModule = IAudioInsightsEditorModule::GetChecked();
#else
		IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
#endif // WITH_EDITOR

		AudioModulationDashboardViewFactory = MakeShared<AudioModulationInsights::FAudioModulationDashboardViewFactory>();
		AudioInsightsModule.RegisterDashboardViewFactory(AudioModulationDashboardViewFactory.ToSharedRef());

		AudioInsightsModule.RegisterEventLogCategories(
		{
			{
				"Modulation",
				{
					"Modulator Activated",
					"Modulator Deactivated"
				}
			}
		});

		AudioInsightsModule.RegisterEventLogDisplayNames(
		{
			{ TEXT("Modulation"),            LOCTEXT("EventLogCategory_Modulation", "Modulation") },
			{ TEXT("Modulator Activated"),   LOCTEXT("EventLogTraceMessage_ModulatorActivated", "Modulator Activated") },
			{ TEXT("Modulator Deactivated"), LOCTEXT("EventLogTraceMessage_ModulatorDeactivated", "Modulator Deactivated") },
		});
	}
}

void FAudioModulationInsightsModule::ShutdownModule()
{
	if (!IsRunningCommandlet())
	{	
	#if WITH_EDITOR
		if (!IAudioInsightsEditorModule::IsModuleLoaded())
		{
			return;
		}

		IAudioInsightsEditorModule& AudioInsightsModule = IAudioInsightsEditorModule::GetChecked();
	#else
		if (!IAudioInsightsModule::IsModuleLoaded())
		{
			return;
		}

		IAudioInsightsModule& AudioInsightsModule = IAudioInsightsModule::GetChecked();
	#endif // WITH_EDITOR

		if (AudioModulationDashboardViewFactory.IsValid())
		{
			AudioInsightsModule.UnregisterDashboardViewFactory(AudioModulationDashboardViewFactory->GetName());
		}
	}
}

IMPLEMENT_MODULE(FAudioModulationInsightsModule, AudioModulationInsights);

#undef LOCTEXT_NAMESPACE
