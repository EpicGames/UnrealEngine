// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioModulationInsightsStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

FAudioModulationInsightsStyle::FAudioModulationInsightsStyle()
	: FSlateStyleSet(GetStyleName())
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	// Register style content paths
	const TSharedPtr<const IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	if (Plugin.IsValid())
	{
		SetContentRoot(Plugin->GetBaseDir() / "Resources");
	}

	SetCoreContentRoot(FPaths::EngineContentDir() / "Slate");

	// Icons
	const FVector2D Icon16(16.0f, 16.0f);

	Set("ControlBus", new IMAGE_BRUSH(TEXT("control_bus"), Icon16));
	Set("ControlBusMix", new IMAGE_BRUSH(TEXT("control_bus_mix"), Icon16));

	// Register slate style
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAudioModulationInsightsStyle::~FAudioModulationInsightsStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

const FAudioModulationInsightsStyle& FAudioModulationInsightsStyle::Get()
{
	static FAudioModulationInsightsStyle AudioModulationInsightsStyle;
	return AudioModulationInsightsStyle;
}

const FName& FAudioModulationInsightsStyle::GetStyleName()
{
	static const FName StyleName = "AudioModulationInsightsStyle";
	return StyleName;
}

FSlateIcon FAudioModulationInsightsStyle::CreateIcon(const FName& InName) const
{
	return { GetStyleName(), InName };
}

#undef LOCTEXT_NAMESPACE
