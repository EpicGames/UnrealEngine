// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

class FAudioModulationInsightsStyle final : public FSlateStyleSet
{
public:
	FAudioModulationInsightsStyle();
	~FAudioModulationInsightsStyle();
	
	static const FAudioModulationInsightsStyle& Get();
	static const FName& GetStyleName();

	FSlateIcon CreateIcon(const FName& InName) const;
};
