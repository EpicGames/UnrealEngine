// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationTraceMessages.h"

#include "Trace/TraceWriter.h"
#include "TraceWriter/CacheWriteHandler.h"

namespace AudioModulationInsights
{
	using UE::Audio::Insights::IAudioCachedMessage;

	UE::Audio::Insights::FCacheWriteHandler FActivateModulatorTraceMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("ActivateModulatorTraceMessage"))
					.Field(ANSITEXTVIEW("DeviceId"),      UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),     UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("ModulatorId"),   UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ModulatorType"), UE::Trace::ETraceWriterFieldType::Uint8)
					.Field(ANSITEXTVIEW("Name"),          UE::Trace::ETraceWriterFieldType::WideString)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FActivateModulatorTraceMessage& Msg = static_cast<const FActivateModulatorTraceMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",      Msg.DeviceId)
					.Field("Timestamp",     TimestampCycles)
					.Field("ModulatorId",   static_cast<uint32>(Msg.ModulatorId))
					.Field("ModulatorType", static_cast<uint8>(Msg.ModulatorType))
					.Field("Name",          FStringView(Msg.ModulatorName))
					.End();
			}
		};
	}

	UE::Audio::Insights::FCacheWriteHandler FUpdateModulatorTraceMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("UpdateModulatorTraceMessage"))
					.Field(ANSITEXTVIEW("DeviceId"),                  UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),                 UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("ModulatorId"),               UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("ModulatorValue"),            UE::Trace::ETraceWriterFieldType::Float32)
					.Field(ANSITEXTVIEW("IsBypassed"),                UE::Trace::ETraceWriterFieldType::Bool)
					.Field(ANSITEXTVIEW("ContributingModulatorIds"),  UE::Trace::ETraceWriterFieldType::Uint32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
					.Field(ANSITEXTVIEW("ContributingModulatorValues"), UE::Trace::ETraceWriterFieldType::Float32 | UE::Trace::ETraceWriterFieldType::ArrayFlag)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FUpdateModulatorTraceMessage& Msg = static_cast<const FUpdateModulatorTraceMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",       Msg.DeviceId)
					.Field("Timestamp",      TimestampCycles)
					.Field("ModulatorId",    static_cast<uint32>(Msg.ModulatorId))
					.Field("ModulatorValue", Msg.ModulatorValue)
					.Field("IsBypassed",     Msg.bIsBypassed)
					.Field("ContributingModulatorIds",    TConstArrayView<uint32>(Msg.ContributingModulatorIds.GetData(), Msg.ContributingModulatorIds.Num()))
					.Field("ContributingModulatorValues", TConstArrayView<float>(Msg.ContributingModulatorValues.GetData(), Msg.ContributingModulatorValues.Num()))
					.End();
			}
		};
	}

	UE::Audio::Insights::FCacheWriteHandler FDeactivateModulatorTraceMessage::GetCacheWriteHandler() const
	{
		return {
			[](UE::Trace::FTraceWriter& Writer) -> uint32
			{
				return Writer.DeclareEvent(ANSITEXTVIEW("Audio"), ANSITEXTVIEW("DeactivateModulatorTraceMessage"))
					.Field(ANSITEXTVIEW("DeviceId"),    UE::Trace::ETraceWriterFieldType::Uint32)
					.Field(ANSITEXTVIEW("Timestamp"),   UE::Trace::ETraceWriterFieldType::Uint64)
					.Field(ANSITEXTVIEW("ModulatorId"), UE::Trace::ETraceWriterFieldType::Uint32)
					.End();
			},
			[](UE::Trace::FTraceWriter& Writer, uint32 EventId, const IAudioCachedMessage& BaseMsg, uint64 TimestampCycles)
			{
				const FDeactivateModulatorTraceMessage& Msg = static_cast<const FDeactivateModulatorTraceMessage&>(BaseMsg);

				Writer.WriteEvent(EventId)
					.Field("DeviceId",    Msg.DeviceId)
					.Field("Timestamp",   TimestampCycles)
					.Field("ModulatorId", static_cast<uint32>(Msg.ModulatorId))
					.End();
			}
		};
	}
} // namespace AudioModulationInsights
