// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "AudioModulationDataTypes.h"
#include "Cache/IAudioCachedMessage.h"
#include "Trace/Analyzer.h"

namespace AudioModulationInsights
{
	namespace ModulatorTraceMessageNames
	{
		inline const FName ActivateModulatorTraceMessage(TEXT("ActivateModulatorTraceMessage"));
		inline const FName UpdateModulatorTraceMessage(TEXT("UpdateModulatorTraceMessage"));
		inline const FName DeactivateModulatorTraceMessage(TEXT("DeactivateModulatorTraceMessage"));
	}

	// Mirrors MODULATOR_TRACE_MESSAGE_BASE: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/AudioModulationSystem.cpp
	struct FModulatorTraceMessageBase : public UE::Audio::Insights::IAudioCachedMessage
	{
		FModulatorTraceMessageBase() = default;
		FModulatorTraceMessageBase(const UE::Trace::IAnalyzer::FOnEventContext& InContext)
		{
			const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;
			Timestamp = InContext.EventTime.AsSeconds(EventData.GetValue<uint64>("Timestamp"));
			DeviceId = static_cast<::Audio::FDeviceId>(EventData.GetValue<uint32>("DeviceId"));
			ModulatorId = EventData.GetValue<FModulatorId>("ModulatorId");
		}

		// from UE::Audio::Insights::IAudioCachedMessage
		virtual uint64 GetID() const override { return ModulatorId; }
		virtual uint32 GetSizeOf() const override { return sizeof(FModulatorTraceMessageBase); }

		Audio::FDeviceId DeviceId = INDEX_NONE;
		FModulatorId ModulatorId = INDEX_NONE;
	};

	// Mirrors ActivateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/AudioModulationSystem.cpp
	struct FActivateModulatorTraceMessage final : public FModulatorTraceMessageBase
	{
		FActivateModulatorTraceMessage() = default;
		FActivateModulatorTraceMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext) : FModulatorTraceMessageBase(InContext)
		{
			const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;
			ModulatorType = EventData.GetValue<EModulatorTraceType>("ModulatorType");
			EventData.GetString("Name", ModulatorName);
		}

		// from UE::Audio::Insights::IAudioCachedMessage
		virtual uint32 GetSizeOf() const override { return (sizeof(FActivateModulatorTraceMessage) + ModulatorName.GetAllocatedSize()); }
		virtual const FName GetMessageName() const override { return ModulatorTraceMessageNames::ActivateModulatorTraceMessage; }
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;

		EModulatorTraceType ModulatorType { EModulatorTraceType::COUNT };
		FString ModulatorName;
	};

	// Mirrors UpdateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/AudioModulationSystem.cpp
	struct FUpdateModulatorTraceMessage final : public FModulatorTraceMessageBase
	{
		FUpdateModulatorTraceMessage() = default;
		FUpdateModulatorTraceMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext) : FModulatorTraceMessageBase(InContext)
		{
			const UE::Trace::IAnalyzer::FEventData& EventData = InContext.EventData;

			ModulatorValue = EventData.GetValue<float>("ModulatorValue");
			bIsBypassed = EventData.GetValue<bool>("IsBypassed");

			const TArrayView<const FModulatorId> ReadOnlyContributingModulatorIds = EventData.GetArrayView<FModulatorId>("ContributingModulatorIds");
			for (int32 Index = 0; Index < ReadOnlyContributingModulatorIds.Num(); ++Index)
			{
				ContributingModulatorIds.Emplace(ReadOnlyContributingModulatorIds[Index]);
			}

			const TArrayView<const float> ReadOnlyContributingModulatorValues = EventData.GetArrayView<float>("ContributingModulatorValues");
			for (int32 Index = 0; Index < ReadOnlyContributingModulatorValues.Num(); ++Index)
			{
				ContributingModulatorValues.Emplace(ReadOnlyContributingModulatorValues[Index]);
			}
		}

		// from UE::Audio::Insights::IAudioCachedMessage
		virtual uint32 GetSizeOf() const override { return (sizeof(FUpdateModulatorTraceMessage) + ContributingModulatorIds.GetAllocatedSize() + ContributingModulatorValues.GetAllocatedSize()); }
		virtual const FName GetMessageName() const override { return ModulatorTraceMessageNames::UpdateModulatorTraceMessage; }
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;

		float ModulatorValue { 1.0f };
		bool bIsBypassed { false };
		TArray<FModulatorId> ContributingModulatorIds;
		TArray<float> ContributingModulatorValues;
	};

	// Mirrors DeactivateModulatorTraceMessage: Engine/Plugins/Runtime/AudioModulation/Source/AudioModulation/Private/AudioModulationSystem.cpp
	struct FDeactivateModulatorTraceMessage final : public FModulatorTraceMessageBase
	{
		FDeactivateModulatorTraceMessage() = default;
		FDeactivateModulatorTraceMessage(const UE::Trace::IAnalyzer::FOnEventContext& InContext) : FModulatorTraceMessageBase(InContext) {}

		// from UE::Audio::Insights::IAudioCachedMessage
		virtual const FName GetMessageName() const override { return ModulatorTraceMessageNames::DeactivateModulatorTraceMessage; }
		virtual UE::Audio::Insights::FCacheWriteHandler GetCacheWriteHandler() const override;
	};

} // namespace AudioModulationInsights
