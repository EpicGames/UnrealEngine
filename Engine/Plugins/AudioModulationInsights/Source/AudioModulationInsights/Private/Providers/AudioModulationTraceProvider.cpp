// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationTraceProvider.h"

#include "AudioInsightsUtils.h"
#include "Cache/AudioInsightsCacheManager.h"
#include "IAudioInsightsModule.h"

namespace AudioModulationInsights
{
	const FAudioModulationDashboardEntry& FAudioModulationDashboardEntry::CastEntry(const UE::Audio::Insights::IDashboardDataViewEntry& InDataViewEntry)
	{
		return static_cast<const FAudioModulationDashboardEntry&>(InDataViewEntry);
	};

	FAudioModulationDashboardEntry& FAudioModulationDashboardEntry::CastEntryMutable(UE::Audio::Insights::IDashboardDataViewEntry& InDataViewEntry)
	{
		return static_cast<FAudioModulationDashboardEntry&>(InDataViewEntry);
	};

	UE::Trace::IAnalyzer* FAudioModulationTraceProvider::ConstructAnalyzer(TraceServices::IAnalysisSession& InSession)
	{
		class FAudioModulationTraceAnalyzer final : public UE::Audio::Insights::FTraceProviderBase::FTraceAnalyzerBase
		{
		public:
			FAudioModulationTraceAnalyzer(TSharedRef<FAudioModulationTraceProvider> InProvider, TraceServices::IAnalysisSession& InSession)
				: UE::Audio::Insights::FTraceProviderBase::FTraceAnalyzerBase(InProvider)
				, Session(InSession)
			{
			}

			virtual void OnAnalysisBegin(const UE::Trace::IAnalyzer::FOnAnalysisContext& Context) override
			{
				UE::Audio::Insights::FTraceProviderBase::FTraceAnalyzerBase::OnAnalysisBegin(Context);
				
				UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
				Builder.RouteEvent(RouteId_ActivateModulatorTraceMessage, "Audio", "ActivateModulatorTraceMessage");
				Builder.RouteEvent(RouteId_UpdateModulatorTraceMessage, "Audio", "UpdateModulatorTraceMessage");
				Builder.RouteEvent(RouteId_DeactivateModulatorTraceMessage, "Audio", "DeactivateModulatorTraceMessage");
			}

			virtual bool OnHandleEvent(uint16 RouteId, UE::Trace::IAnalyzer::EStyle Style, const UE::Trace::IAnalyzer::FOnEventContext& Context) override
			{
				LLM_SCOPE_BYNAME(TEXT("Insights/FAudioModulationTraceAnalyzer"));

				FAudioModulationTraceProvider& Provider = GetProvider<FAudioModulationTraceProvider>();
				switch (RouteId)
				{
					case RouteId_ActivateModulatorTraceMessage:
					{
						CacheMessage<FActivateModulatorTraceMessage>(Context, Provider.ActivateModulatorTraceMessages);
						break;
					}

					case RouteId_UpdateModulatorTraceMessage:
					{
						CacheMessage<FUpdateModulatorTraceMessage>(Context, Provider.UpdateModulatorTraceMessages);
						break;
					}

					case RouteId_DeactivateModulatorTraceMessage:
					{
						CacheMessage<FDeactivateModulatorTraceMessage>(Context, Provider.DeactivateModulatorTraceMessages);
						break;
					}

					default:
					{
						return OnEventFailure(RouteId, Style, Context);
					}
				}

				{
					TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
					const double Timestamp = Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("Timestamp"));
					Session.UpdateDurationSeconds(Timestamp);
				}

				return OnEventSuccess(RouteId, Style, Context);
			}

		private:
			enum : uint16
			{
				RouteId_ActivateModulatorTraceMessage,
				RouteId_UpdateModulatorTraceMessage,
				RouteId_DeactivateModulatorTraceMessage
			};

			TraceServices::IAnalysisSession& Session;
		};

		return new FAudioModulationTraceAnalyzer(AsShared(), InSession);
	}

	TSharedPtr<FAudioModulationDashboardEntry>* FAudioModulationTraceProvider::FindOrAddEntry(const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage)
	{
		TSharedPtr<FAudioModulationDashboardEntry>& FoundEntry = FindOrAddDeviceEntry(InActivateModulatorTraceMessage.DeviceId, InActivateModulatorTraceMessage.ModulatorId);

		if (!FoundEntry.IsValid())
		{
			FoundEntry = MakeShared<FAudioModulationDashboardEntry>();
			FoundEntry->DeviceId = InActivateModulatorTraceMessage.DeviceId;
			FoundEntry->ModulatorId = InActivateModulatorTraceMessage.ModulatorId;
		}

		return &FoundEntry;
	}

	void FAudioModulationTraceProvider::ProcessActivateEntry(const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
	{
		if (!OutAudioModulationDashboardEntry || !OutAudioModulationDashboardEntry->IsValid())
		{
			return;
		}

		FAudioModulationDashboardEntry& AudioModulationEntry = *OutAudioModulationDashboardEntry->Get();

		AudioModulationEntry.Timestamp = InActivateModulatorTraceMessage.Timestamp;
		AudioModulationEntry.DeviceId = InActivateModulatorTraceMessage.DeviceId;
		AudioModulationEntry.ModulatorId = InActivateModulatorTraceMessage.ModulatorId;
		AudioModulationEntry.ModulatorType = InActivateModulatorTraceMessage.ModulatorType;

		ensure((!InActivateModulatorTraceMessage.ModulatorName.IsEmpty()));
		AudioModulationEntry.Name = InActivateModulatorTraceMessage.ModulatorName;
		AudioModulationEntry.DisplayName = UE::Audio::Insights::AudioInsightsUtils::ResolveObjectDisplayName(AudioModulationEntry.Name);
	}

	void FAudioModulationTraceProvider::ProcessUpdateEntry(const FUpdateModulatorTraceMessage& InUpdateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
	{
		if (!OutAudioModulationDashboardEntry || !OutAudioModulationDashboardEntry->IsValid())
		{
			return;
		}

		FAudioModulationDashboardEntry& AudioModulationEntry = *OutAudioModulationDashboardEntry->Get();

		// This allows us to handle modulator contributors being added to a currently active modulator. E.g. adding generators to an active control bus.
		AudioModulationEntry.Timestamp = InUpdateModulatorTraceMessage.Timestamp;
		AudioModulationEntry.Value = InUpdateModulatorTraceMessage.ModulatorValue;
		AudioModulationEntry.bIsBypassed = InUpdateModulatorTraceMessage.bIsBypassed;
		AudioModulationEntry.ContributingModulatorIds = InUpdateModulatorTraceMessage.ContributingModulatorIds;
		AudioModulationEntry.ContributingModulatorValues = InUpdateModulatorTraceMessage.ContributingModulatorValues;
	}

	void FAudioModulationTraceProvider::ProcessDeactivateEntry(const FDeactivateModulatorTraceMessage& InDeactivateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
	{
		if (!OutAudioModulationDashboardEntry || !OutAudioModulationDashboardEntry->IsValid())
		{
			return;
		}

		FAudioModulationDashboardEntry& AudioModulationEntry = *OutAudioModulationDashboardEntry->Get();
	
		if (AudioModulationEntry.Timestamp < InDeactivateModulatorTraceMessage.Timestamp)
		{
			RemoveDeviceEntry(InDeactivateModulatorTraceMessage.DeviceId, InDeactivateModulatorTraceMessage.ModulatorId);
		}
	}

	bool FAudioModulationTraceProvider::ProcessMessages()
	{
		ensure(IsInGameThread());

		ProcessMessageQueue<FActivateModulatorTraceMessage>(ActivateModulatorTraceMessages, 
			[this](const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage)
			{
				return FindOrAddEntry(InActivateModulatorTraceMessage);
			},
			[this](const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage, TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
			{
				ProcessActivateEntry(InActivateModulatorTraceMessage, OutAudioModulationDashboardEntry);
			});

		ProcessMessageQueue<FUpdateModulatorTraceMessage>(UpdateModulatorTraceMessages,
			[this](const FUpdateModulatorTraceMessage& InUpdateModulatorTraceMessage)
			{
				return FindDeviceEntry(InUpdateModulatorTraceMessage.DeviceId, InUpdateModulatorTraceMessage.ModulatorId);
			},
			[this](const FUpdateModulatorTraceMessage& InUpdateModulatorTraceMessage, TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
			{
				ProcessUpdateEntry(InUpdateModulatorTraceMessage, OutAudioModulationDashboardEntry);
			});

		ProcessMessageQueue<FDeactivateModulatorTraceMessage>(DeactivateModulatorTraceMessages, 
			[this](const FDeactivateModulatorTraceMessage& InDeactivateModulatorTraceMessage)
			{
				return FindDeviceEntry(InDeactivateModulatorTraceMessage.DeviceId, InDeactivateModulatorTraceMessage.ModulatorId);
			},
			[this](const FDeactivateModulatorTraceMessage& InDeactivateModulatorTraceMessage, TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry)
			{
				ProcessDeactivateEntry(InDeactivateModulatorTraceMessage, OutAudioModulationDashboardEntry);
			});

		return true;
	}

	void FAudioModulationTraceProvider::OnTimingViewTimeMarkerChanged(const double InTimeMarker)
	{
		DeviceDataMap.Reset();

		const UE::Audio::Insights::FAudioInsightsCacheManager& AudioInsightsCacheManager = IAudioInsightsModule::GetChecked().GetCacheManager();

		AudioInsightsCacheManager.IterateTo<FActivateModulatorTraceMessage>(ModulatorTraceMessageNames::ActivateModulatorTraceMessage, InTimeMarker, [this, InTimeMarker](const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage)
		{
			TSharedPtr<FAudioModulationDashboardEntry>* const AudioModulationDashboardEntry = FindOrAddEntry(InActivateModulatorTraceMessage);
			ProcessActivateEntry(InActivateModulatorTraceMessage, AudioModulationDashboardEntry);
		});

		AudioInsightsCacheManager.IterateOverRange<FUpdateModulatorTraceMessage>(ModulatorTraceMessageNames::UpdateModulatorTraceMessage, InTimeMarker, (InTimeMarker + CacheHistoryLimitSec), [this](const FUpdateModulatorTraceMessage& InUpdateModulatorTraceMessage)
		{
			TSharedPtr<FAudioModulationDashboardEntry>* const AudioModulationDashboardEntry = FindDeviceEntry(InUpdateModulatorTraceMessage.DeviceId, InUpdateModulatorTraceMessage.ModulatorId);
			ProcessUpdateEntry(InUpdateModulatorTraceMessage, AudioModulationDashboardEntry);
		});

		AudioInsightsCacheManager.IterateTo<FDeactivateModulatorTraceMessage>(ModulatorTraceMessageNames::DeactivateModulatorTraceMessage, InTimeMarker, [this, InTimeMarker](const FDeactivateModulatorTraceMessage& InDeactivateModulatorTraceMessage)
		{
			TSharedPtr<FAudioModulationDashboardEntry>* const AudioModulationDashboardEntry = FindDeviceEntry(InDeactivateModulatorTraceMessage.DeviceId, InDeactivateModulatorTraceMessage.ModulatorId);
			ProcessDeactivateEntry(InDeactivateModulatorTraceMessage, AudioModulationDashboardEntry);
		});

		// Call parent method to update LastMessageId
		FTraceProviderBase::OnTimingViewTimeMarkerChanged(InTimeMarker);
	}

	void FAudioModulationTraceProvider::OnTimeControlMethodReset()
	{
		// When leaving the cache here, we intentionally clear entries, to avoid deactivated runtime modulators staying in the dashboard
		// Since modulators update regularly, any active modulators will be added back in within ProcessMessages()
		Reset();
	}

	TSharedPtr<FAudioModulationDashboardEntry> FAudioModulationTraceProvider::FindEntry(const FModulatorId InModulatorId) const
	{
		const FDeviceData* const DeviceData = FindFilteredDeviceData();
		if (!DeviceData)
		{
			return nullptr;
		}

		const TSharedPtr<FAudioModulationDashboardEntry>* const FoundEntry = DeviceData->Find(InModulatorId);
		if (!FoundEntry)
		{
			return nullptr;
		}

		return *FoundEntry;
	}

} // namespace AudioModulationInsights
