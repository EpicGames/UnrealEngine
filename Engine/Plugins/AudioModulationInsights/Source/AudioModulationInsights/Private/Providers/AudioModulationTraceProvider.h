// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioInsightsTraceProviderBase.h"
#include "AudioModulationDataTypes.h"
#include "Messages/AnalyzerMessageQueue.h"
#include "Messages/AudioModulationTraceMessages.h"
#include "Views/TableDashboardViewFactory.h"

namespace AudioModulationInsights
{
	class FAudioModulationDashboardEntry final : public UE::Audio::Insights::FSoundAssetDashboardEntry
	{
	public:
		FAudioModulationDashboardEntry() = default;
		~FAudioModulationDashboardEntry() = default;

		static const FAudioModulationDashboardEntry& CastEntry(const UE::Audio::Insights::IDashboardDataViewEntry& InDataViewEntry);
		static FAudioModulationDashboardEntry& CastEntryMutable(UE::Audio::Insights::IDashboardDataViewEntry& InDataViewEntry);

		FText GetDisplayNameAsFText() const { return FText::FromString(DisplayName); }

		bool bIsBypassed { false };

		FModulatorId ModulatorId = INDEX_NONE;
		EModulatorTraceType ModulatorType { EModulatorTraceType::COUNT };

		float Value { 1.0f };
		float ContributingValue { 1.0f };

		TArray<FModulatorId> ContributingModulatorIds;
		TArray<float> ContributingModulatorValues;

		FString DisplayName;
	};

	class FAudioModulationTraceProvider final
		: public UE::Audio::Insights::TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioModulationDashboardEntry>>
		, public TSharedFromThis<FAudioModulationTraceProvider>
	{
	public:
		FAudioModulationTraceProvider()
			: UE::Audio::Insights::TDeviceDataMapTraceProvider<uint32, TSharedPtr<FAudioModulationDashboardEntry>>(GetName_Static())
		{}

		static FName GetName_Static() { return "AudioModulationTraceProvider"; };
		TSharedPtr<FAudioModulationDashboardEntry> FindEntry(const FModulatorId InModulatorId) const;

	private:
		// from UE::Audio::Insights::FTraceProviderBase via UE::Audio::Insights::TDeviceDataMapTraceProvider
		virtual UE::Trace::IAnalyzer* ConstructAnalyzer(TraceServices::IAnalysisSession& InSession) override;
		virtual bool ProcessMessages() override;
		virtual void OnTimingViewTimeMarkerChanged(const double InTimeMarker) override;
		virtual void OnTimeControlMethodReset() override;

		TSharedPtr<FAudioModulationDashboardEntry>* FindOrAddEntry(const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage);

		void ProcessActivateEntry(const FActivateModulatorTraceMessage& InActivateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry);
		void ProcessUpdateEntry(const FUpdateModulatorTraceMessage& InUpdateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry);
		void ProcessDeactivateEntry(const FDeactivateModulatorTraceMessage& InDeactivateModulatorTraceMessage, const TSharedPtr<FAudioModulationDashboardEntry>* const OutAudioModulationDashboardEntry);

		static constexpr double CacheHistoryLimitSec { 0.1 };
		UE::Audio::Insights::TAnalyzerMessageQueue<FActivateModulatorTraceMessage> ActivateModulatorTraceMessages { CacheHistoryLimitSec };
		UE::Audio::Insights::TAnalyzerMessageQueue<FUpdateModulatorTraceMessage> UpdateModulatorTraceMessages { CacheHistoryLimitSec };
		UE::Audio::Insights::TAnalyzerMessageQueue<FDeactivateModulatorTraceMessage> DeactivateModulatorTraceMessages { CacheHistoryLimitSec };
	};

} // namespace AudioModulationInsights
