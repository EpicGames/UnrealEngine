// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationContributorsDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "AudioModulationInsightsCommands.h"
#include "AudioModulationInsightsStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "Internationalization/Text.h"
#include "Providers/AudioModulationTraceProvider.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

namespace AudioModulationInsights
{
	FAudioModulationContributorsDashboardViewFactory::FAudioModulationContributorsDashboardViewFactory()
	{
		FAudioModulationInsightsCommands::Register();
		BindCommands();
	}

	FAudioModulationContributorsDashboardViewFactory::~FAudioModulationContributorsDashboardViewFactory()
	{
		FAudioModulationInsightsCommands::Unregister();
	}

	FName FAudioModulationContributorsDashboardViewFactory::GetName() const
	{
		return "ModulationContributors";
	};

	FText FAudioModulationContributorsDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioInsights_AudioModulationContributors_DisplayName", "ModulationContributors");
	};

	FSlateIcon FAudioModulationContributorsDashboardViewFactory::GetIcon() const
	{
		return FAudioModulationInsightsStyle::Get().CreateIcon("ControlBusMix");
	};
	
	UE::Audio::Insights::EDefaultDashboardTabStack FAudioModulationContributorsDashboardViewFactory::GetDefaultTabStack() const 
	{ 
		return UE::Audio::Insights::EDefaultDashboardTabStack::Analysis;
	};

	void FAudioModulationContributorsDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();

		const FAudioModulationInsightsCommands& Commands = FAudioModulationInsightsCommands::Get();

#if WITH_EDITOR
		CommandList->MapAction(Commands.GetBrowseCommand(), FExecuteAction::CreateLambda([this]() { BrowseToAsset(); }));
		CommandList->MapAction(Commands.GetEditCommand(), FExecuteAction::CreateLambda([this]() { OpenAsset(); }));
#endif // WITH_EDITOR
	}

	TSharedRef<SWidget> FAudioModulationContributorsDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<SWidget> TableDashboardWidget = UE::Audio::Insights::FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs);

		SearchBoxWidget->SetVisibility(EVisibility::Hidden);

		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->SetSelectionMode(ESelectionMode::SingleToggle);
		}

		return TableDashboardWidget;
	}

	const TMap<FName, UE::Audio::Insights::FTraceTableDashboardViewFactory::FColumnData>& FAudioModulationContributorsDashboardViewFactory::GetColumns() const
	{
		/*
		* Note: BugHawk flags this function with the following message:
		* 
		* "GetColumns() returns a function-local static TMap initialized via CreateColumnData(), but CreateColumnData and/or the column callbacks capture this."
		* "Because the static map persists for the lifetime of the process, the stored lambdas can retain a dangling this pointer after the first factory instance is destroyed,
		* "causing use-after-free when the callbacks are invoked and/or incorrectly binding subsequent instances to the first instance's state."
		*
		* While this is a great catch, we only ever have a single instances of our AudioInsights dashboards.
		* Ideally we enforce this by implementing a single instance only pattern e.g. singletons.
		*
		* But until this is rolled out across the AudioInsights codebase, we should 'in theory' be safe to ignore this BugHawk catch (famous last words).
		*/
		auto CreateColumnData = [this]()
		{
			using namespace UE::Audio::Insights;

			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					ModulatorColumnNames::ModulatorIDColumnName,
					{
						LOCTEXT("AudioModulationContributors_ModulatorIDColumnName", "Modulator ID"),
						[](const IDashboardDataViewEntry& InData) { return FText::AsNumber(FAudioModulationDashboardEntry::CastEntry(InData).ModulatorId); },
						nullptr	/* GetIconName */,
						true /* bDefaultHidden */,
						0.08f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorNameColumnName,
					{
						LOCTEXT("AudioModulationContributors_ModulatorNameColumnName", "Modulator Name"),
						[](const IDashboardDataViewEntry& InData) { return FText::FromString(FAudioModulationDashboardEntry::CastEntry(InData).DisplayName); },
						nullptr	/* GetIconName */,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorTypeColumnName,
					{
						LOCTEXT("AudioModulationContributors_ModulatorTypeColumnName", "Modulator Type"),
						[](const IDashboardDataViewEntry& InData)
						{
							return GetModulatorTraceTypeName(FAudioModulationDashboardEntry::CastEntry(InData).ModulatorType);
						},
						[](const IDashboardDataViewEntry& InData)
						{
							switch (FAudioModulationDashboardEntry::CastEntry(InData).ModulatorType) 
							{
								case EModulatorTraceType::ControlBus: return "AudioInsights.Icon.Modulation.ControlBus";
								case EModulatorTraceType::ControlBusMix: return "AudioInsights.Icon.Modulation.ControlBusMix";
								case EModulatorTraceType::Generator: return "AudioInsights.Icon.Modulation.Generator";
								case EModulatorTraceType::ParameterPatch: return "AudioInsights.Icon.Modulation.ParameterPatch";
								case EModulatorTraceType::COUNT: break;
							}

							return "";
						},
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorValueColumnName,
					{
						LOCTEXT("AudioModulationContributors_ModulatorValueColumnName", "Modulator Value"),
						[this](const IDashboardDataViewEntry& InData) 
						{
							const FAudioModulationDashboardEntry& CastedInData = FAudioModulationDashboardEntry::CastEntry(InData);

							if (CastedInData.ModulatorId == MainWindowSelectedModulatorId)
							{
								const bool bIsControlBusMix = (CastedInData.ModulatorType == EModulatorTraceType::ControlBusMix);
								return FText::Format(LOCTEXT("AudioModulationContributors_FinalValue", "{0} {1}"),
									bIsControlBusMix ? FText::GetEmpty() : FText::AsNumber(CastedInData.Value, FSlateStyle::Get().GetLinearVolumeFloatFormat()),
									bIsControlBusMix ? FText::GetEmpty() : FText::FromString("(Final)"));
							}

							return FText::AsNumber(CastedInData.ContributingValue, FSlateStyle::Get().GetLinearVolumeFloatFormat());
						},
						nullptr	/* GetIconName */,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorBypassedColumnName,
					{
						LOCTEXT("AudioModulationContributors_ModulatorBypassedColumnName", "Bypassed"),
						[](const IDashboardDataViewEntry& InData) { return FText::GetEmpty(); },
						[](const IDashboardDataViewEntry& InData) { return FAudioModulationDashboardEntry::CastEntry(InData).bIsBypassed ? "AudioInsights.Icon.CheckBoxCheck" : "AudioInsights.Icon.CheckBoxIndeterminate"; },
						false /* bDefaultHidden */,
						0.08f /* FillWidth */
					}
				}
			};
		};

		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		return ColumnData;
	}

	void FAudioModulationContributorsDashboardViewFactory::ResetEntries()
	{
		DataViewEntries.Reset();
		RefreshFilteredEntriesListView();
	}

	void FAudioModulationContributorsDashboardViewFactory::UpdateEntries(TArray<TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>> InEntries)
	{
		DataViewEntries = InEntries;
		RefreshFilteredEntriesListView();
	}

	TSharedPtr<SWidget> FAudioModulationContributorsDashboardViewFactory::OnConstructContextMenu()
	{
		const FAudioModulationInsightsCommands& Commands = FAudioModulationInsightsCommands::Get();

		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.BeginSection("AudioModulationContributorsDashboardActions", LOCTEXT("AudioModulationContributorsActions_HeaderText", "Modulation Options"));
		{
			MenuBuilder.AddMenuEntry(Commands.GetBrowseCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), UE::Audio::Insights::FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Browse"));
			MenuBuilder.AddMenuEntry(Commands.GetEditCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), UE::Audio::Insights::FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Edit"));
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void FAudioModulationContributorsDashboardViewFactory::OnSelectionChanged(TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		UE::Audio::Insights::FTraceObjectTableDashboardViewFactory::OnSelectionChanged(SelectedItem, SelectInfo);

		switch (SelectInfo)
		{
			// Note: Only call ResetSelectedModulatorId() for user interactions not via non-user calls with ESelectInfo::Type::Direct
			// This maintains our selection when interacting with other dashboards / scrubbing through the cache etc
			case ESelectInfo::Type::OnKeyPress:
			case ESelectInfo::Type::OnNavigation:
			case ESelectInfo::Type::OnMouseClick:
			{
				ResetSelectedModulatorId();
			} break;

			case ESelectInfo::Type::Direct: break;
		}

		if (SelectedItem.IsValid())
		{
			FAudioModulationDashboardEntry& AudioModulationEntry = FAudioModulationDashboardEntry::CastEntryMutable(*SelectedItem.Get());
			SelectedModulatorId = AudioModulationEntry.ModulatorId;
		}
	}

	void FAudioModulationContributorsDashboardViewFactory::ResetSelectedModulatorId()
	{
		SelectedModulatorId = InvalidModulatorId;
	}

	void FAudioModulationContributorsDashboardViewFactory::OnMainWindowModulatorSelectionChanged(const FModulatorId InSelectedModulatorId)
	{
		MainWindowSelectedModulatorId = InSelectedModulatorId;
	}

	FSlateColor FAudioModulationContributorsDashboardViewFactory::GetRowColor(const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& InRowDataPtr)
	{
		const FAudioModulationDashboardEntry& CastedInData = FAudioModulationDashboardEntry::CastEntry(*InRowDataPtr);

		if (CastedInData.ModulatorId == MainWindowSelectedModulatorId)
		{
			return ModulatorColors::SelectedModulator;
		}

		if (CastedInData.ModulatorId == SelectedModulatorId)
		{
			return ModulatorColors::SelectedContributor;
		}

		return ModulatorColors::Default;
	}

} // namespace AudioModulationInsights

#undef LOCTEXT_NAMESPACE
