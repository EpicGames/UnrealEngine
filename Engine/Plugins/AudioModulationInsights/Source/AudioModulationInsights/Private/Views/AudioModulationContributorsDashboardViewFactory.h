// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioModulationDataTypes.h"
#include "Templates/SharedPointer.h"
#include "Views/TableDashboardViewFactory.h"

namespace AudioModulationInsights
{
	class FAudioModulationContributorsDashboardViewFactory final : public UE::Audio::Insights::FTraceObjectTableDashboardViewFactory
	{
	public:

		FAudioModulationContributorsDashboardViewFactory();
		~FAudioModulationContributorsDashboardViewFactory();

		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;

		void ResetEntries();
		void UpdateEntries(TArray<TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>> InEntries);
		FModulatorId GetSelectedModulatorId() const { return SelectedModulatorId; }
		void OnMainWindowModulatorSelectionChanged(const FModulatorId InSelectedModulatorId);
		void ResetSelectedModulatorId();

	private:
		virtual FName GetName() const override;
		virtual FText GetDisplayName() const override;
		virtual FSlateIcon GetIcon() const override;
		virtual UE::Audio::Insights::EDefaultDashboardTabStack GetDefaultTabStack() const override;
		virtual TSharedPtr<SWidget> OnConstructContextMenu() override;
		virtual const TMap<FName, FColumnData>& GetColumns() const override;
		virtual void OnSelectionChanged(TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;
		virtual FSlateColor GetRowColor(const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& InRowDataPtr) override;

		// Note: Intentionally not supporting sorting of the contributors. This ensures our selected modulator and final value to always be displayed at the top.
		virtual void SortTable() override {};

		void BindCommands();
		TSharedPtr<FUICommandList> CommandList;

		FModulatorId SelectedModulatorId = InvalidModulatorId;
		FModulatorId MainWindowSelectedModulatorId = InvalidModulatorId;
	};

} // namespace AudioModulationInsights
