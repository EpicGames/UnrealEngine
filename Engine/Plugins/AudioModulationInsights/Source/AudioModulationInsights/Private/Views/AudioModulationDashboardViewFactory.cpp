// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationDashboardViewFactory.h"

#include "AudioInsightsStyle.h"
#include "AudioInsightsTraceModule.h"
#include "AudioInsightsTraceProviderBase.h"
#include "AudioModulationContributorsDashboardViewFactory.h"
#include "AudioModulationInsightsCommands.h"
#include "AudioModulationInsightsStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "IAudioInsightsModule.h"
#include "Internationalization/Text.h"
#include "Providers/AudioModulationTraceProvider.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

namespace AudioModulationInsights
{
	FAudioModulationDashboardViewFactory::FAudioModulationDashboardViewFactory()
	{
		IAudioInsightsModule& InsightsModule = IAudioInsightsModule::GetChecked();
		IAudioInsightsTraceModule& InsightsTraceModule = InsightsModule.GetTraceModule();

		ContributorsDashboard = MakeShared<FAudioModulationContributorsDashboardViewFactory>();
		ensure(ContributorsDashboard.IsValid());

		AudioModulationProvider = MakeShared<FAudioModulationTraceProvider>();
		ensure(AudioModulationProvider.IsValid());

		InsightsTraceModule.AddTraceProvider(StaticCastSharedPtr<UE::Audio::Insights::FTraceProviderBase>(AudioModulationProvider));

		Providers =
		{
			AudioModulationProvider
		};

		FAudioModulationInsightsCommands::Register();

		BindCommands();
	}

	FAudioModulationDashboardViewFactory::~FAudioModulationDashboardViewFactory()
	{
		FAudioModulationInsightsCommands::Unregister();
	}

	FName FAudioModulationDashboardViewFactory::GetName() const
	{
		return "Modulation";
	}

	FText FAudioModulationDashboardViewFactory::GetDisplayName() const
	{
		return LOCTEXT("AudioInsights_ModulationAudioModulation_DisplayName", "Modulation");
	}

	void FAudioModulationDashboardViewFactory::ProcessEntries(UE::Audio::Insights::FTraceTableDashboardViewFactory::EProcessReason Reason)
	{
		UpdateContributorWatchWindow();
		FilterByModulatorName();
		FilterByModulatorType();

		if (IsModulatorSelected())
		{
			const bool bSelectedEntryVisible = DataViewEntries.ContainsByPredicate([this](const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& Entry)
			{
				const FAudioModulationDashboardEntry& AudioModulationEntry = FAudioModulationDashboardEntry::CastEntry(*Entry);
				return (AudioModulationEntry.ModulatorId == SelectedModulatorId);
			});

			if (!bSelectedEntryVisible)
			{
				ResetSelectedModulatorId();

				if (FilteredEntriesListView.IsValid())
				{
					FilteredEntriesListView->ClearHighlightedItems();
				}

				if (ContributorsDashboard)
				{
					ContributorsDashboard->ResetEntries();
				}
			}
		}
	}
	
	void FAudioModulationDashboardViewFactory::UpdateContributorWatchWindow()
	{
		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->ClearHighlightedItems();
		}

		if (!ContributorsDashboard)
		{
			return;
		}

		TSharedPtr<FAudioModulationDashboardEntry> SelectedAudioModulationEntry = AudioModulationProvider->FindEntry(SelectedModulatorId);

		if (!SelectedAudioModulationEntry)
		{
			ResetSelectedModulatorId();
			ContributorsDashboard->ResetEntries();
			return;
		}
		
		ContributingEntries.Reset();
		ContributingEntries.Emplace(SelectedAudioModulationEntry);

		ensure(SelectedAudioModulationEntry->ContributingModulatorIds.Num() == SelectedAudioModulationEntry->ContributingModulatorValues.Num());

		const int32 MinNumContributors = FMath::Min(SelectedAudioModulationEntry->ContributingModulatorIds.Num(), SelectedAudioModulationEntry->ContributingModulatorValues.Num());

		for (int32 ContributingModulatorIndex = 0; ContributingModulatorIndex < MinNumContributors; ++ContributingModulatorIndex)
		{
			const FModulatorId ContributorModulatorId = SelectedAudioModulationEntry->ContributingModulatorIds[ContributingModulatorIndex];

			// Note: Filtering affects the DataViewEntries/FilteredEntriesListView. Therefore we must search for the contributing modulator within the unfiltered provider container
			// E.g. Filter set to Control Buses, clicking a Control Bus, we still want to be able to find all the contributors such as generators
			// DataViewEntries/FilteredEntriesListView will have filtered out the generators, but the Provider will still have a full record of all Modulators
			TSharedPtr<FAudioModulationDashboardEntry> ContributorEntry = AudioModulationProvider->FindEntry(ContributorModulatorId);

			if (!ContributorEntry)
			{
				continue;
			}

			if (FilteredEntriesListView.IsValid())
			{
				FilteredEntriesListView->SetItemHighlighted(ContributorEntry, true);
			}

			ContributorEntry->ContributingValue = SelectedAudioModulationEntry->ContributingModulatorValues[ContributingModulatorIndex];

			/*
			* Note: It's possible for a modulator to contain many instances of the same contributor
			* E.g. A Control Bus can have generator LFO_1 listed more than once
			* E.g. A Parameter Patch can have multiple instances of the same Control Bus
			* Therefore we must make a copy of already existing entries, rather than adding multiple instances of the same shared pointer
			* This also prevents a crash when DataViewEntries contains 2 or more instances of the same shared pointer
			*/
			if (ContributingEntries.ContainsByPredicate([ContributorModulatorId](const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& ExistingEntry)
				{
					const FAudioModulationDashboardEntry& ExistingAudioModulationEntry = FAudioModulationDashboardEntry::CastEntry(*ExistingEntry);
					return (ExistingAudioModulationEntry.ModulatorId == ContributorModulatorId);
				}))
			{
				const FAudioModulationDashboardEntry AudioModulationEntryToCopy = FAudioModulationDashboardEntry::CastEntry(*ContributorEntry.Get());
				TSharedPtr<FAudioModulationDashboardEntry> AudioModulationEntryCopy = MakeShared<FAudioModulationDashboardEntry>(AudioModulationEntryToCopy);
				ContributingEntries.Emplace(AudioModulationEntryCopy);
			}
			else
			{
				ContributingEntries.Emplace(ContributorEntry);
			}
		}

		ContributorsDashboard->UpdateEntries(ContributingEntries);
	}

	void FAudioModulationDashboardViewFactory::FilterByModulatorName()
	{
		const FString FilterString = GetSearchFilterText().ToString();
		FTraceTableDashboardViewFactory::FilterEntries<FAudioModulationTraceProvider>([&FilterString](const UE::Audio::Insights::IDashboardDataViewEntry& Entry)
		{
			const FAudioModulationDashboardEntry& AudioModulationEntry = FAudioModulationDashboardEntry::CastEntry(Entry);

			return AudioModulationEntry.DisplayName.Contains(FilterString);
		});
	}

	void FAudioModulationDashboardViewFactory::FilterByModulatorType()
	{
		TArray<TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>> EntriesToFilterOut;

		const EModulatorTraceType ComboBoxSelectedModulatorTypesEnum = ComboBoxSelectedModulatorTypes.IsValid() ? ComboBoxSelectedModulatorTypes->Key : EModulatorTraceType::COUNT;

		for (const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& Entry : DataViewEntries)
		{
			if (Entry.IsValid())
			{
				const FAudioModulationDashboardEntry& AudioModulationEntry = FAudioModulationDashboardEntry::CastEntry(*Entry);

				if (ComboBoxSelectedModulatorTypesEnum != EModulatorTraceType::COUNT)
				{
					if ((ComboBoxSelectedModulatorTypesEnum == EModulatorTraceType::ControlBus && AudioModulationEntry.ModulatorType != EModulatorTraceType::ControlBus) ||
						(ComboBoxSelectedModulatorTypesEnum == EModulatorTraceType::ControlBusMix && AudioModulationEntry.ModulatorType != EModulatorTraceType::ControlBusMix) ||
						(ComboBoxSelectedModulatorTypesEnum == EModulatorTraceType::Generator && AudioModulationEntry.ModulatorType != EModulatorTraceType::Generator) ||
						(ComboBoxSelectedModulatorTypesEnum == EModulatorTraceType::ParameterPatch && AudioModulationEntry.ModulatorType != EModulatorTraceType::ParameterPatch))
					{
						EntriesToFilterOut.Emplace(Entry);
					}
				}
			}
		}

		for (const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& Entry : EntriesToFilterOut)
		{
			DataViewEntries.Remove(Entry);
		}
	}

	FSlateIcon FAudioModulationDashboardViewFactory::GetIcon() const
	{
		return FAudioModulationInsightsStyle::Get().CreateIcon("ControlBusMix");
	}

	UE::Audio::Insights::EDefaultDashboardTabStack FAudioModulationDashboardViewFactory::GetDefaultTabStack() const
	{
		return UE::Audio::Insights::EDefaultDashboardTabStack::Analysis;
	}

	TSharedRef<SWidget> FAudioModulationDashboardViewFactory::MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedRef<SWidget> TableDashboardWidget = UE::Audio::Insights::FTraceTableDashboardViewFactory::MakeWidget(OwnerTab, SpawnTabArgs);
		TSharedRef<SWidget> ContributorsDashboardWidget = ContributorsDashboard->MakeWidget(OwnerTab, SpawnTabArgs);

		if (FilteredEntriesListView.IsValid())
		{
			FilteredEntriesListView->SetSelectionMode(ESelectionMode::SingleToggle);
		}

		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 0.0f, 0.0f, 6.0f)
			[
				MakeModulatorTypeFilterWidget()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FLinearColor::Black)
				[
					SNew(SSpacer)
					.Size(FVector2D(0.0f, 0.1f))
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				// Dashboard and contributors dashboard
				SNew(SSplitter)
					.Orientation(Orient_Horizontal)
					+ SSplitter::Slot()
					.Value(0.5f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						[
							TableDashboardWidget
						]
					]
					+ SSplitter::Slot()
					.Value(0.5f)
					[
						SNew(SVerticalBox)
						.Visibility_Lambda([this]()
						{
							return IsContributorWindowVisible() ? EVisibility::Visible : EVisibility::Collapsed;
						})
						+ SVerticalBox::Slot()
						[
							ContributorsDashboardWidget
						]
					]
			];
	}

	const TMap<FName, UE::Audio::Insights::FTraceTableDashboardViewFactory::FColumnData>& FAudioModulationDashboardViewFactory::GetColumns() const
	{
		auto CreateColumnData = []()
		{
			using namespace UE::Audio::Insights;

			return TMap<FName, FTraceTableDashboardViewFactory::FColumnData>
			{
				{
					ModulatorColumnNames::ModulatorIDColumnName,
					{
						LOCTEXT("AudioModulation_ModulatorIDColumnName", "Modulator ID"),
						[](const IDashboardDataViewEntry& InData) { return FText::AsNumber(FAudioModulationDashboardEntry::CastEntry(InData).ModulatorId); },
						nullptr,
						true /* bDefaultHidden */,
						0.08f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorNameColumnName,
					{
						LOCTEXT("AudioModulation_ModulatorNameColumnName", "Modulator Name"),
						[](const IDashboardDataViewEntry& InData) { return FText::FromString(FAudioModulationDashboardEntry::CastEntry(InData).DisplayName); },
						nullptr	/* GetIconName */,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorTypeColumnName,
					{
						LOCTEXT("AudioModulation_ModulatorTypeColumnName", "Modulator Type"),
						[](const IDashboardDataViewEntry& InData)
						{
							return GetModulatorTraceTypeName(FAudioModulationDashboardEntry::CastEntry(InData).ModulatorType);
						},
						[](const IDashboardDataViewEntry& InData)
						{
							switch (FAudioModulationDashboardEntry::CastEntry(InData).ModulatorType) 
							{
								case EModulatorTraceType::ControlBus: return "AudioInsights.Icon.Modulation.ControlBus";
								case EModulatorTraceType::ControlBusMix: return "AudioInsights.Icon.Modulation.ControlBusMix";
								case EModulatorTraceType::Generator: return "AudioInsights.Icon.Modulation.Generator";
								case EModulatorTraceType::ParameterPatch: return "AudioInsights.Icon.Modulation.ParameterPatch";
								case EModulatorTraceType::COUNT: break;
							}

							return "";
						},
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorValueColumnName,
					{
						LOCTEXT("AudioModulation_ModulatorValueColumnName", "Modulator Value"),
						[](const IDashboardDataViewEntry& InData) 
						{
							const FAudioModulationDashboardEntry& CastedInData = FAudioModulationDashboardEntry::CastEntry(InData);
							if (CastedInData.ModulatorType == EModulatorTraceType::ControlBusMix)
							{
								return FText() /* LOCTEXT("AudioModulation_ModulatorValueBusMix", "") */;
							}
							
							return FText::AsNumber(CastedInData.Value, FSlateStyle::Get().GetLinearVolumeFloatFormat()); 
						},
						nullptr	/* GetIconName */,
						false /* bDefaultHidden */,
						0.15f /* FillWidth */
					}
				},
				{
					ModulatorColumnNames::ModulatorBypassedColumnName,
					{
						LOCTEXT("AudioModulation_ModulatorBypassedColumnName", "Bypassed"),
						[](const IDashboardDataViewEntry& InData) { return FText::GetEmpty(); },
						[](const IDashboardDataViewEntry& InData) { return FAudioModulationDashboardEntry::CastEntry(InData).bIsBypassed ? "AudioInsights.Icon.CheckBoxCheck" : "AudioInsights.Icon.CheckBoxIndeterminate"; },
						false /* bDefaultHidden */,
						0.08f /* FillWidth */
					}
				}
			};
		};
		static const TMap<FName, FTraceTableDashboardViewFactory::FColumnData> ColumnData = CreateColumnData();
		return ColumnData;
	}

	TSharedRef<SWidget> FAudioModulationDashboardViewFactory::MakeModulatorTypeFilterWidget()
	{
		if (ComboBoxModulatorTypes.IsEmpty())
		{
			ComboBoxModulatorTypes.Emplace(MakeShared<FComboBoxSelectionItem>(EModulatorTraceType::COUNT, LOCTEXT("AudioModulationComboBoxFilter_ModulatorTypeAll", "Modulator Type: All")));
			ComboBoxModulatorTypes.Emplace(MakeShared<FComboBoxSelectionItem>(EModulatorTraceType::ControlBus, LOCTEXT("AudioModulationComboBoxFilter_ModulatorTypeBus", "Modulator Type: Control Bus")));
			ComboBoxModulatorTypes.Emplace(MakeShared<FComboBoxSelectionItem>(EModulatorTraceType::ControlBusMix, LOCTEXT("AudioModulationComboBoxFilter_ModulatorTypeBusMix", "Modulator Type: Bus Mix")));
			ComboBoxModulatorTypes.Emplace(MakeShared<FComboBoxSelectionItem>(EModulatorTraceType::Generator, LOCTEXT("AudioModulationComboBoxFilter_ModulatorTypeGenerator", "Modulator Type: Generator")));
			ComboBoxModulatorTypes.Emplace(MakeShared<FComboBoxSelectionItem>(EModulatorTraceType::ParameterPatch, LOCTEXT("AudioModulationComboBoxFilter_ModulatorTypePatch", "Modulator Type: Parameter Patch")));

			ComboBoxSelectedModulatorTypes = ComboBoxModulatorTypes[0];
		}

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Margin(FMargin(0.0, 2.0, 0.0, 0.0))
			]
			+ SHorizontalBox::Slot()
			.MaxWidth(2.0f)
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(SComboBox<TSharedPtr<FComboBoxSelectionItem>>)
				.OptionsSource(&ComboBoxModulatorTypes)
				.OnGenerateWidget_Lambda([this](const TSharedPtr<FComboBoxSelectionItem>& ModulatingSourceTypePtr)
				{
					const FText ModulatingSourceTypeDisplayName = ModulatingSourceTypePtr.IsValid() ? ModulatingSourceTypePtr->Value /*DisplayName*/ : FText::GetEmpty();

					return SNew(STextBlock)
						.Text(ModulatingSourceTypeDisplayName);
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<FComboBoxSelectionItem> InComboBoxSelectedModulatorTypesPtr, ESelectInfo::Type)
				{
					if (InComboBoxSelectedModulatorTypesPtr.IsValid())
					{
						ComboBoxSelectedModulatorTypes = InComboBoxSelectedModulatorTypesPtr;
						UpdateFilterReason = EProcessReason::FilterUpdated;
					}
				})
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						const int32 FoundIndex = ComboBoxModulatorTypes.Find(ComboBoxSelectedModulatorTypes);
						if (ComboBoxModulatorTypes.IsValidIndex(FoundIndex) && ComboBoxModulatorTypes[FoundIndex].IsValid())
						{
							return ComboBoxModulatorTypes[FoundIndex]->Value;
						}

						return FText::GetEmpty();
					})
				]
			];
	}

	void FAudioModulationDashboardViewFactory::SortTable()
	{
		using namespace UE::Audio::Insights;

		if (SortByColumn == ModulatorColumnNames::ModulatorIDColumnName)
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return AData.ModulatorId < BData.ModulatorId;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return BData.ModulatorId < AData.ModulatorId;
				});
			}
		}
		else if (SortByColumn == ModulatorColumnNames::ModulatorNameColumnName)
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return AData.DisplayName.ToLower() < BData.DisplayName.ToLower();
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return BData.DisplayName.ToLower() < AData.DisplayName.ToLower();
				});
			}
		}
		else if (SortByColumn == ModulatorColumnNames::ModulatorTypeColumnName)
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return AData.ModulatorType < BData.ModulatorType;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return BData.ModulatorType < AData.ModulatorType;
				});
			}
		}
		else if (SortByColumn == ModulatorColumnNames::ModulatorValueColumnName)
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return AData.Value < BData.Value;
				});
			}
			else if (SortMode == EColumnSortMode::Descending)
			{
				DataViewEntries.Sort([](const TSharedPtr<IDashboardDataViewEntry>& A, const TSharedPtr<IDashboardDataViewEntry>& B)
				{
					const FAudioModulationDashboardEntry& AData = FAudioModulationDashboardEntry::CastEntry(*A.Get());
					const FAudioModulationDashboardEntry& BData = FAudioModulationDashboardEntry::CastEntry(*B.Get());

					return BData.Value < AData.Value;
				});
			}
		}
	}

	TSharedPtr<SWidget> FAudioModulationDashboardViewFactory::OnConstructContextMenu()
	{
		const FAudioModulationInsightsCommands& Commands = FAudioModulationInsightsCommands::Get();

		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

		MenuBuilder.BeginSection("AudioModulationDashboardActions", LOCTEXT("AudioModulationActions_HeaderText", "Modulation Options"));
		{
			MenuBuilder.AddMenuEntry(Commands.GetBrowseCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), UE::Audio::Insights::FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Browse"));
			MenuBuilder.AddMenuEntry(Commands.GetEditCommand(), NAME_None, TAttribute<FText>(), TAttribute<FText>(), UE::Audio::Insights::FSlateStyle::Get().CreateIcon("AudioInsights.Icon.SoundDashboard.Edit"));
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}	

	FSlateColor FAudioModulationDashboardViewFactory::GetRowColor(const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& InRowDataPtr)
	{
		const FAudioModulationDashboardEntry& CastedInData = FAudioModulationDashboardEntry::CastEntry(*InRowDataPtr);
		if (CastedInData.ModulatorId == SelectedModulatorId)
		{
			return ModulatorColors::SelectedModulator;
		}

		if (IsContributorWindowVisible() && (ContributorsDashboard->GetSelectedModulatorId() == CastedInData.ModulatorId))
		{
			return ModulatorColors::SelectedContributor;
		}

		return ModulatorColors::Default;
	}

	FReply FAudioModulationDashboardViewFactory::OnDataRowKeyInput(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const
	{
		return (CommandList && CommandList->ProcessCommandBindings(InKeyEvent)) ? FReply::Handled() : FReply::Unhandled();
	}

	void FAudioModulationDashboardViewFactory::OnSelectionChanged(TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		UE::Audio::Insights::FTraceObjectTableDashboardViewFactory::OnSelectionChanged(SelectedItem, SelectInfo);

		switch (SelectInfo)
		{
			// Note: Only call ResetSelectedModulatorId() for user interactions not via non-user calls with ESelectInfo::Type::Direct
			// This maintains our selection when interacting with other dashboards / scrubbing through the cache etc
			case ESelectInfo::Type::OnKeyPress:
			case ESelectInfo::Type::OnNavigation:
			case ESelectInfo::Type::OnMouseClick:
			{
				ResetSelectedModulatorId();
			} break;

			case ESelectInfo::Type::Direct: break;
		}

		if (SelectedItem.IsValid())
		{
			FAudioModulationDashboardEntry& AudioModulationEntry = FAudioModulationDashboardEntry::CastEntryMutable(*SelectedItem.Get());
			SelectedModulatorId = AudioModulationEntry.ModulatorId;

			if (ContributorsDashboard)
			{
				ContributorsDashboard->OnMainWindowModulatorSelectionChanged(SelectedModulatorId);
			}
		}

		// Note: Ensure the contributor watch window is updated on selection changed
		// Especially important when reading from the cache & our selection has changed but the cache time marker hasn't
		UpdateContributorWatchWindow();
	}

	bool FAudioModulationDashboardViewFactory::IsModulatorSelected() const
	{
		return (SelectedModulatorId != InvalidModulatorId);
	}

	bool FAudioModulationDashboardViewFactory::IsContributorWindowVisible() const
	{
		return (ContributorsDashboard && IsModulatorSelected());
	}

	void FAudioModulationDashboardViewFactory::ResetSelectedModulatorId()
	{
		SelectedModulatorId = InvalidModulatorId;

		if (ContributorsDashboard)
		{
			ContributorsDashboard->ResetSelectedModulatorId();
		}
	}

	void FAudioModulationDashboardViewFactory::BindCommands()
	{
		CommandList = MakeShared<FUICommandList>();
		ensure(CommandList.IsValid());

		const FAudioModulationInsightsCommands& Commands = FAudioModulationInsightsCommands::Get();

#if WITH_EDITOR
		CommandList->MapAction(Commands.GetBrowseCommand(), FExecuteAction::CreateLambda([this]() { BrowseToAsset(); }));
		CommandList->MapAction(Commands.GetEditCommand(), FExecuteAction::CreateLambda([this]() { OpenAsset(); }));
#endif // WITH_EDITOR
	}
} // namespace AudioModulationInsights

#undef LOCTEXT_NAMESPACE
