// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioModulationDataTypes.h"
#include "Templates/SharedPointer.h"
#include "Views/TableDashboardViewFactory.h"

namespace AudioModulationInsights
{
	class FAudioModulationContributorsDashboardViewFactory;
	class FAudioModulationTraceProvider;

	class FAudioModulationDashboardViewFactory final : public UE::Audio::Insights::FTraceObjectTableDashboardViewFactory
	{
	public:
		FAudioModulationDashboardViewFactory();
		~FAudioModulationDashboardViewFactory();

		virtual FName GetName() const override;
	
	private:
		virtual FText GetDisplayName() const override;
		virtual FSlateIcon GetIcon() const override;
		virtual UE::Audio::Insights::EDefaultDashboardTabStack GetDefaultTabStack() const override;
		virtual void ProcessEntries(UE::Audio::Insights::FTraceTableDashboardViewFactory::EProcessReason Reason) override;
		virtual TSharedRef<SWidget> MakeWidget(TSharedRef<SDockTab> OwnerTab, const FSpawnTabArgs& SpawnTabArgs) override;
		virtual const TMap<FName, UE::Audio::Insights::FTraceTableDashboardViewFactory::FColumnData>& GetColumns() const override;
		virtual void SortTable() override;
		virtual FSlateColor GetRowColor(const TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>& InRowDataPtr) override;
		virtual TSharedPtr<SWidget> OnConstructContextMenu() override;
		virtual FReply OnDataRowKeyInput(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) const override;
		virtual void OnSelectionChanged(TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry> SelectedItem, ESelectInfo::Type SelectInfo) override;

		bool IsModulatorSelected() const;
		bool IsContributorWindowVisible() const;
		void ResetSelectedModulatorId();
		void UpdateContributorWatchWindow();
		void BindCommands();
		void FilterByModulatorName();
		void FilterByModulatorType();
		TSharedRef<SWidget> MakeModulatorTypeFilterWidget();

		using FComboBoxSelectionItem = TPair<EModulatorTraceType, FText>;
		TArray<TSharedPtr<FComboBoxSelectionItem>> ComboBoxModulatorTypes;
		TArray<TSharedPtr<UE::Audio::Insights::IDashboardDataViewEntry>> ContributingEntries;
		
		TSharedPtr<FAudioModulationContributorsDashboardViewFactory> ContributorsDashboard;
		TSharedPtr<FUICommandList> CommandList;
		TSharedPtr<FComboBoxSelectionItem> ComboBoxSelectedModulatorTypes;
		TSharedPtr<FAudioModulationTraceProvider> AudioModulationProvider;

		FModulatorId SelectedModulatorId = InvalidModulatorId;
	};
} // namespace AudioModulationInsights
