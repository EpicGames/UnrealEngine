// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioModulationInsightsCommands.h"

#include "AudioInsightsStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AudioModulationInsights"

namespace AudioModulationInsights
{
	FAudioModulationInsightsCommands::FAudioModulationInsightsCommands()
		: TCommands<FAudioModulationInsightsCommands>("AudioModulationInsightsCommands", LOCTEXT("AudioModulationInsightsCommands_ContextDescText", "Audio Modulation Insights Commands"), NAME_None, UE::Audio::Insights::FSlateStyle::GetStyleName())
	{
	}
	
	void FAudioModulationInsightsCommands::RegisterCommands()
	{
		UI_COMMAND(Browse, "Browse To Asset", "Browses to the selected sound asset in the content browser.", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Control));
		UI_COMMAND(Edit, "Edit", "Opens the selected sound for edit.", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Control));
	}
} // namespace AudioModulationInsights

#undef LOCTEXT_NAMESPACE
