// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

namespace AudioModulationInsights
{
	class FAudioModulationInsightsCommands : public TCommands<FAudioModulationInsightsCommands>
	{
	public:
		FAudioModulationInsightsCommands();

		virtual void RegisterCommands() override;

		TSharedPtr<const FUICommandInfo> GetBrowseCommand() const { return Browse; }
		TSharedPtr<const FUICommandInfo> GetEditCommand() const   { return Edit; }
		
	private:
		TSharedPtr<FUICommandInfo> Browse;
		TSharedPtr<FUICommandInfo> Edit;
	};
} // namespace AudioModulationInsights
