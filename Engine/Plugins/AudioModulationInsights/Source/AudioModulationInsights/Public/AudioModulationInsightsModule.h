// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"
#include "Views/AudioModulationDashboardViewFactory.h"

#define UE_API AUDIOMODULATIONINSIGHTS_API

class FAudioModulationInsightsModule : public IModuleInterface
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

private:
	TSharedPtr<AudioModulationInsights::FAudioModulationDashboardViewFactory> AudioModulationDashboardViewFactory;
};

#undef UE_API
